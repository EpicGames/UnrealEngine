// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class HierarchicalLODUtilities : ModuleRules
{
    public HierarchicalLODUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[]
			{
				"Core",
				"CoreUObject"
			}
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
			{
				"BSPUtils",
				"EditorFramework",
				"Engine",
				"MaterialUtilities",
				"MeshDescription",
				"StaticMeshDescription",
				"UnrealEd",
                "Projects",
			}
        );

		PrivateIncludePathModuleNames.AddRange(
            new string[]
            {
				"GeometryProcessingInterfaces"
			}
        );

        DynamicallyLoadedModuleNames.AddRange(
            new string[]
            {
                "MeshUtilities",
                "MeshMergeUtilities",
                "MeshReductionInterface",
				"GeometryProcessingInterfaces"
			}
        );
	}
}
