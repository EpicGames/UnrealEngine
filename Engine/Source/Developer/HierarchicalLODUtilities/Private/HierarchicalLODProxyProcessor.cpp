// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchicalLODProxyProcessor.h"
#include "Misc/ScopeLock.h"
#include "Engine/StaticMesh.h"
#include "Engine/LODActor.h"
#include "IHierarchicalLODUtilities.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "Editor.h"

#include "Modules/ModuleManager.h"
#include "Interfaces/IProjectManager.h"
#include "StaticMeshResources.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Engine/HLODProxy.h"
#include "HierarchicalLOD.h"
#include "Algo/Transform.h"
#include "Modules/ModuleManager.h"

FHierarchicalLODProxyProcessor::FHierarchicalLODProxyProcessor()
{
#if WITH_EDITOR
	FEditorDelegates::MapChange.AddRaw(this, &FHierarchicalLODProxyProcessor::OnMapChange);
	FEditorDelegates::NewCurrentLevel.AddRaw(this, &FHierarchicalLODProxyProcessor::OnNewCurrentLevel);
#endif // WITH_EDITOR
}

FHierarchicalLODProxyProcessor::~FHierarchicalLODProxyProcessor()
{
#if WITH_EDITOR
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
#endif // WITH_EDITOR
}

bool FHierarchicalLODProxyProcessor::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FHierarchicalLODProxyProcessor_Tick);

	FScopeLock Lock(&StateLock);

	while (ToProcessJobs.Num())
	{
		FProcessData* Data = ToProcessJobs.Pop();
		UStaticMesh* MainMesh = nullptr;		
		for (TStrongObjectPtr<UObject>& AssetObject : Data->AssetObjects)
		{
			// Check if this is the generated proxy (static-)mesh
			UStaticMesh* StaticMesh = Cast<UStaticMesh>(AssetObject.Get());
			if (StaticMesh)
			{
				check(StaticMesh != nullptr);
				MainMesh = StaticMesh;
			}
		}
		check(MainMesh != nullptr);

		// Force lightmap coordinate to 0 for proxy meshes
		MainMesh->SetLightMapCoordinateIndex(0);
		// Trigger post edit change, simulating we made a change in the Static mesh editor (could only call Build, but this is for possible future changes)
		MainMesh->PostEditChange();

		// Set new static mesh, location and sub-objects (UObjects)
		bool bDirtyPackage = false;
		UStaticMesh* PreviousStaticMesh = Data->LODActor->GetStaticMeshComponent()->GetStaticMesh();
		bDirtyPackage |= (PreviousStaticMesh != MainMesh);
		Data->LODActor->SetStaticMesh(MainMesh);
		bDirtyPackage |= (Data->LODActor->GetActorLocation() != FVector::ZeroVector);
		Data->LODActor->SetActorLocation(FVector::ZeroVector);
		
		// Check resulting mesh and give a warning if it exceeds the vertex / triangle cap for certain platforms
		FProjectStatus ProjectStatus;
		if (IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && (ProjectStatus.IsTargetPlatformSupported(TEXT("Android")) || ProjectStatus.IsTargetPlatformSupported(TEXT("IOS"))))
		{
			if (MainMesh->GetRenderData() && MainMesh->GetRenderData()->LODResources.Num() && MainMesh->GetRenderData()->LODResources[0].IndexBuffer.Is32Bit())
			{
				FMessageLog("HLODResults").Warning()
					->AddToken(FUObjectToken::Create(Data->LODActor))
					->AddToken(FTextToken::Create(FText::FromString(TEXT(" Mesh has more that 65535 vertices, incompatible with mobile; forcing 16-bit (will probably cause rendering issues)."))));
				
				FMessageLog("HLODResults").Open();
			}
		}

		// Calculate correct drawing distance according to given screensize
		// At the moment this assumes a fixed field of view of 90 degrees (horizontal and vertical axi)
		static const float FOVRad = 90.0f * (float)PI / 360.0f;
		static const FMatrix ProjectionMatrix = FPerspectiveMatrix(FOVRad, 1920, 1080, 0.01f);
		FBoxSphereBounds Bounds = Data->LODActor->GetStaticMeshComponent()->CalcBounds(FTransform());

		FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		IHierarchicalLODUtilities* Utilities = Module.GetUtilities();
		
		float DrawDistance;
		if (Data->LODSetup.bUseOverrideDrawDistance)
		{
			DrawDistance = Data->LODSetup.OverrideDrawDistance;
		}
		else
		{
			DrawDistance = Utilities->CalculateDrawDistanceFromScreenSize(Bounds.SphereRadius, Data->LODSetup.TransitionScreenSize, ProjectionMatrix);
		}
		
		bDirtyPackage |= (Data->LODActor->GetDrawDistance() != DrawDistance);
		Data->LODActor->SetDrawDistance(DrawDistance);

		Data->LODActor->DetermineShadowingFlags();
		Data->LODActor->UpdateSubActorLODParents();

		// Link proxy to actor
		UHLODProxy* PreviousProxy = Data->LODActor->GetProxy();
		Data->Proxy->AddMesh(Data->LODActor, MainMesh, UHLODProxy::GenerateKeyForActor(Data->LODActor));
		bDirtyPackage |= (Data->LODActor->GetProxy() != PreviousProxy);

		if(bDirtyPackage)
		{
			Data->LODActor->MarkPackageDirty();
		}

		// Clean out standalone meshes from the proxy package as we are about to GC, and mesh merging creates assets that are 
		// supposed to be standalone
		Utilities->CleanStandaloneAssetsInPackage(Data->Proxy->GetOutermost());

		// Collect garbage to clean up old unreferenced data in the HLOD package
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		delete Data;
	}

	return true;
}

FGuid FHierarchicalLODProxyProcessor::AddProxyJob(ALODActor* InLODActor, UHLODProxy* InProxy, const FHierarchicalSimplification& LODSetup)
{
	FScopeLock Lock(&StateLock);
	check(InLODActor);
	// Create new unique Guid which will be used to identify this job
	FGuid JobGuid = FGuid::NewGuid();
	// Set up processing data
	FProcessData* Data = new FProcessData();
	Data->LODActor = InLODActor;
	Data->Proxy = InProxy;
	Data->LODSetup = LODSetup;	

	JobActorMap.Add(JobGuid, Data);

	return JobGuid;
}

void FHierarchicalLODProxyProcessor::ProcessProxy(const FGuid InGuid, TArray<UObject*>& InAssetsToSync)
{
	FScopeLock Lock(&StateLock);

	// Check if there is data stored for the given Guid
	FProcessData** DataPtr = JobActorMap.Find(InGuid);
	if (DataPtr)
	{
		// If so push the job onto the processing queue
		FProcessData* Data = *DataPtr;
		JobActorMap.Remove(InGuid);
		if (Data && Data->LODActor && InAssetsToSync.Num())
		{
			Algo::Transform(InAssetsToSync, Data->AssetObjects, [](UObject* InItem){ return TStrongObjectPtr<UObject>(InItem); });
			ToProcessJobs.Push(Data);
		}
	}
}

FCreateProxyDelegate& FHierarchicalLODProxyProcessor::GetCallbackDelegate()
{
	// Bind ProcessProxy to the delegate (if not bound yet)
	if (!CallbackDelegate.IsBound())
	{
		CallbackDelegate.BindRaw(this, &FHierarchicalLODProxyProcessor::ProcessProxy);
	}

	return CallbackDelegate;
}

bool FHierarchicalLODProxyProcessor::IsProxyGenerationRunning() const
{
	return JobActorMap.Num() > 0 || ToProcessJobs.Num() > 0;
}

void FHierarchicalLODProxyProcessor::OnMapChange(uint32 MapFlags)
{
	ClearProcessingData();
}

void FHierarchicalLODProxyProcessor::OnNewCurrentLevel()
{
	ClearProcessingData();
}

void FHierarchicalLODProxyProcessor::ClearProcessingData()
{
	FScopeLock Lock(&StateLock);
	JobActorMap.Empty();
	ToProcessJobs.Empty();
}
