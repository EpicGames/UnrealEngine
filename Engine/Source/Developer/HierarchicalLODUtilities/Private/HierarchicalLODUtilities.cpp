// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchicalLODUtilities.h"
#include "GameFramework/Actor.h"
#include "Components/StaticMeshComponent.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/LODActor.h"
#include "Components/BrushComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Model.h"
#include "Engine/Polys.h"
#include "HierarchicalLODUtilitiesModule.h"

#include "MeshUtilities.h"
#include "StaticMeshResources.h"
#include "HierarchicalLODVolume.h"

#include "Interfaces/IProjectManager.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#include "BSPOps.h"
#include "Builders/CubeBuilder.h"

#include "AssetRegistry/AssetRegistryModule.h" 
#include "Engine/LevelStreaming.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ScopedTransaction.h"
#include "PackageTools.h"
#include "Settings/EditorExperimentalSettings.h"
#endif // WITH_EDITOR

#include "MaterialUtilities.h"
#include "IGeometryProcessingInterfacesModule.h"
#include "GeometryProcessingInterfaces/ApproximateActors.h"

#include "HierarchicalLODProxyProcessor.h"
#include "IMeshReductionManagerModule.h"
#include "MeshMergeModule.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "ObjectTools.h"
#include "Engine/HLODProxy.h"
#include "HierarchicalLOD.h"
#include "LevelUtils.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "UObject/ICookInfo.h"
#include "HLOD/HLODSetup.h"

DEFINE_LOG_CATEGORY_STATIC(LogHierarchicalLODUtilities, Verbose, All);

#define LOCTEXT_NAMESPACE "HierarchicalLODUtils"

void FHierarchicalLODUtilities::ExtractSubActorsFromLODActor(AActor* Actor, TArray<AActor*>& InOutActors)
{
	ALODActor* LODActor = CastChecked<ALODActor>(Actor);
	for (AActor* ChildActor : LODActor->SubActors)
	{
		TArray<AActor*> ChildActors;
		if (ChildActor->IsA<ALODActor>())
		{
			ExtractSubActorsFromLODActor(ChildActor, ChildActors);
		}
		else
		{
			ChildActors.Add(ChildActor);
		}

		InOutActors.Append(ChildActors);
	}
}

float FHierarchicalLODUtilities::CalculateScreenSizeFromDrawDistance(const float SphereRadius, const FMatrix& ProjectionMatrix, const float Distance)
{
	return ComputeBoundsScreenSize(FVector::ZeroVector, SphereRadius, FVector(0.0f, 0.0f, Distance), ProjectionMatrix);
}

float FHierarchicalLODUtilities::CalculateDrawDistanceFromScreenSize(const float SphereRadius, const float ScreenSize, const FMatrix& ProjectionMatrix)
{
	return ComputeBoundsDrawDistance(ScreenSize, SphereRadius, ProjectionMatrix);
}

static FString GetHLODProxyName(const FString& InLevelPackageName, const uint32 InHLODLevelIndex)
{
	const FString BaseName = FPackageName::GetShortName(InLevelPackageName);
	return FString::Printf(TEXT("%s_%i_HLOD"), *BaseName, InHLODLevelIndex);
}

static FString GetHLODProxyName(const ULevel* InLevel, const uint32 InHLODLevelIndex)
{
	UPackage* LevelOuterMost = InLevel->GetOutermost();
	const FString PackageName = LevelOuterMost->GetPathName();
	return GetHLODProxyName(PackageName, InHLODLevelIndex);
}

static FString GetHLODPackageName(const FString& InLevelPackageName, const uint32 InHLODLevelIndex, FString& InOutHLODProxyName)
{
	const FString PathName = FPackageName::GetLongPackagePath(InLevelPackageName);
	InOutHLODProxyName = GetHLODProxyName(InLevelPackageName, InHLODLevelIndex);
	return FString::Printf(TEXT("%s/HLOD/%s"), *PathName, *InOutHLODProxyName);
}

FString FHierarchicalLODUtilities::GetWildcardOfHLODPackagesForPackage(const FString& PackageName)
{
	const FString PathName = FPackageName::GetLongPackagePath(PackageName);
	const FString BaseName = FPackageName::GetShortName(PackageName);
	return FString::Printf(TEXT("%s/HLOD/%s_*_HLOD"), *PathName, *BaseName);
}

static FString GetHLODPackageName(const ULevel* InLevel, const uint32 InHLODLevelIndex, FString& InOutHLODProxyName)
{
	// Strip out any PIE or level instance prefix from the given level package name
	FString LevelPackageName;
	if (ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(InLevel))
	{
		LevelPackageName = (StreamingLevel->PackageNameToLoad != NAME_None) ? StreamingLevel->PackageNameToLoad.ToString() : StreamingLevel->GetWorldAssetPackageName();
	}
	else
	{
		LevelPackageName = InLevel->GetOutermost()->GetPathName();
	}
	
	if (InLevel->GetWorld() && InLevel->GetWorld()->IsPlayInEditor())
	{
		LevelPackageName = UWorld::StripPIEPrefixFromPackageName(LevelPackageName, InLevel->GetWorld()->StreamingLevelsPrefix);
	}

	// Build the HLOD package name from the cleaned up level package name
	return GetHLODPackageName(LevelPackageName, InHLODLevelIndex, InOutHLODProxyName);
}

void FHierarchicalLODUtilities::CleanStandaloneAssetsInPackage(UPackage* InPackage)
{
	TArray<UObject*> Objects;
	GetObjectsWithOuter(InPackage, Objects);
	for(UObject* PackageObject : Objects)
	{
		if(PackageObject->HasAnyFlags(RF_Standalone))
		{
			if( PackageObject->IsA<UStaticMesh>() ||
				PackageObject->IsA<UTexture>() ||
				PackageObject->IsA<UMaterialInterface>())
			{
				PackageObject->ClearFlags(RF_Standalone);
			}
		}
	}
}

UHLODProxy* FHierarchicalLODUtilities::CreateOrRetrieveLevelHLODProxy(const ULevel* InLevel, const uint32 HLODLevelIndex)
{
	UPackage* HLODPackage = CreateOrRetrieveLevelHLODPackage(InLevel, HLODLevelIndex);

	// Check if our asset exists
	const FString HLODProxyName = GetHLODProxyName(InLevel, HLODLevelIndex);
	UHLODProxy* Proxy = FindObject<UHLODProxy>(HLODPackage, *HLODProxyName);

	// Get the world associated with this level
	UWorld* LevelWorld = UWorld::FindWorldInPackage(InLevel->GetOutermost());

	// If proxy doesn't exist or is pointing to another world (could happen if package is duplicated)
	if(Proxy == nullptr || Proxy->GetMap() != LevelWorld)
	{
		// Make sure that the package doesn't have any standalone meshes etc. (i.e. this is an old style package)
		CleanStandaloneAssetsInPackage(HLODPackage);

		// Create the new asset
		Proxy = NewObject<UHLODProxy>(HLODPackage, *HLODProxyName, RF_Public | RF_Standalone);
		Proxy->SetMap(LevelWorld);
	}

	return Proxy;	
}

UPackage* FHierarchicalLODUtilities::CreateOrRetrieveLevelHLODPackage(const ULevel* InLevel, const uint32 HLODLevelIndex)
{
	checkf(InLevel != nullptr, TEXT("Invalid Level supplied"));

	FString HLODProxyName;
	const FString HLODLevelPackageName = GetHLODPackageName(InLevel, HLODLevelIndex, HLODProxyName);

	// Find existing package
	bool bCreatedNewPackage = false;
	UPackage* HLODPackage = CreatePackage( *HLODLevelPackageName);
	HLODPackage->FullyLoad();
	HLODPackage->SetPackageFlags(PKG_ContainsMapData);		// PKG_ContainsMapData required so FEditorFileUtils::GetDirtyContentPackages can treat this as a map package

	// Target PackagePath; this is a hack to avoid save file dialog when we will be saving HLOD map package
	HLODPackage->SetLoadedPath(FPackagePath::FromPackageNameChecked(HLODLevelPackageName));

	return HLODPackage;
}

UHLODProxy* FHierarchicalLODUtilities::RetrieveLevelHLODProxy(const ULevel* InLevel, const uint32 HLODLevelIndex)
{
	checkf(InLevel != nullptr, TEXT("Invalid Level supplied"));
	FString HLODProxyName;
	FCookLoadScope CookLoadScope(ECookLoadType::UsedInGame);
	const FString HLODLevelPackageName = GetHLODPackageName(InLevel, HLODLevelIndex, HLODProxyName);

	UHLODProxy* HLODProxy = LoadObject<UHLODProxy>(nullptr, *HLODLevelPackageName, nullptr, LOAD_Quiet | LOAD_NoWarn);
	return HLODProxy;
}

UPackage* FHierarchicalLODUtilities::RetrieveLevelHLODPackage(const ULevel* InLevel, const uint32 HLODLevelIndex)
{
	UHLODProxy* Proxy = RetrieveLevelHLODProxy(InLevel, HLODLevelIndex);
	if(Proxy)
	{
		return Proxy->GetOutermost();
	}
	return nullptr;
}

FString FHierarchicalLODUtilities::GetLevelHLODProxyName(const FString& InLevelPackageName, const uint32 InHLODLevelIndex)
{
	FString HLODProxyName;
	FString HLODPackageName = GetHLODPackageName(InLevelPackageName, InHLODLevelIndex, HLODProxyName);
	return HLODPackageName + TEXT(".") + HLODProxyName;
}

static FString GetImposterMeshName(const UStaticMesh* InImposterMesh)
{
	UPackage* MeshOuterMost = InImposterMesh->GetOutermost();

	const FString BaseName = FPackageName::GetShortName(MeshOuterMost->GetPathName());
	return FString::Printf(TEXT("%s_ImposterMesh"), *BaseName);
}

static FString GetImposterMeshPackageName(const UStaticMesh* InImposterMesh)
{
	UPackage* MeshOuterMost = InImposterMesh->GetOutermost();

	const FString PathName = FPackageName::GetLongPackagePath(MeshOuterMost->GetPathName());
	const FString BaseName = FPackageName::GetShortName(MeshOuterMost->GetPathName());
	return FString::Printf(TEXT("%s/%s_ImposterMesh"), *PathName, *BaseName);
}

static UPackage* CreateOrRetrieveImposterMeshPackage(const UStaticMesh* InImposterMesh)
{
	checkf(InImposterMesh != nullptr, TEXT("Invalid mesh supplied"));

	const FString MeshPackageName = GetImposterMeshPackageName(InImposterMesh);

	UPackage* MeshPackage = CreatePackage( *MeshPackageName);
	MeshPackage->FullyLoad();

	// Target PackagePath; this is a hack to avoid save file dialog when we will be saving imposter mesh package
	MeshPackage->SetLoadedPath(FPackagePath::FromPackageNameChecked(MeshPackageName));

	return MeshPackage;
}

static UMaterialInterface* GetImposterMaterial(UStaticMeshComponent* InComponent)
{
	// Retrieve imposter material
	const int32 LODIndex = InComponent->GetStaticMesh()->GetNumLODs() - 1;

	// Retrieve the sections, we're expect 1 for imposter meshes
	const FStaticMeshSectionArray& Sections = InComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndex].Sections;
	check(Sections.Num() == 1);

	// Retrieve material for this section
	return InComponent->GetMaterial(Sections[0].MaterialIndex);
}

static UStaticMesh* CreateImposterStaticMesh(UStaticMeshComponent* InComponent, const FMeshProxySettings& InProxySettings)
{
	UPackage* ImposterStaticMeshPackage = CreateOrRetrieveImposterMeshPackage(InComponent->GetStaticMesh());

	const UStaticMesh* SourceImposterStaticMesh = InComponent->GetStaticMesh();
	const FVector SourcePositiveBoundsExtension = SourceImposterStaticMesh->GetPositiveBoundsExtension();
	const FVector SourceNegativeBoundsExtension = SourceImposterStaticMesh->GetNegativeBoundsExtension();
	const bool SourceHasBoundsExtension = !SourcePositiveBoundsExtension.IsZero() || !SourceNegativeBoundsExtension.IsZero();

	// check if our asset exists
	const FString ImposterStaticMeshName = GetImposterMeshName(SourceImposterStaticMesh);
	UStaticMesh* ImposterStaticMesh = FindObject<UStaticMesh>(ImposterStaticMeshPackage, *ImposterStaticMeshName);
	bool bMeshChanged = false;

	FMeshDescription SourceMeshDesc;
	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	MeshMergeUtilities.RetrieveMeshDescription(InComponent->GetStaticMesh(), InComponent->GetStaticMesh()->GetNumLODs() - 1, SourceMeshDesc);

	if (ImposterStaticMesh == nullptr)
	{
		// Create the UStaticMesh object.
		ImposterStaticMesh = NewObject<UStaticMesh>(ImposterStaticMeshPackage, *ImposterStaticMeshName, RF_Public | RF_Standalone);
		ImposterStaticMesh->InitResources();

		// make sure it has a new lighting guid
		ImposterStaticMesh->SetLightingGuid();

		// Set it to use textured lightmaps. Note that Build Lighting will do the error-checking (texcoordindex exists for all LODs, etc).
		ImposterStaticMesh->SetLightMapResolution(InProxySettings.LightMapResolution);
		ImposterStaticMesh->SetLightMapCoordinateIndex(1);

		// Add one LOD for the base mesh
		ImposterStaticMesh->SetNumSourceModels(0);
		FStaticMeshSourceModel& SrcModel = ImposterStaticMesh->AddSourceModel();
		/*Don't allow the engine to recalculate normals*/
		SrcModel.BuildSettings.bRecomputeNormals = false;
		SrcModel.BuildSettings.bRecomputeTangents = false;
		SrcModel.BuildSettings.bComputeWeightedNormals = true;
		SrcModel.BuildSettings.bRemoveDegenerates = true;
		SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
		SrcModel.BuildSettings.bUseFullPrecisionUVs = false;
		SrcModel.BuildSettings.bGenerateLightmapUVs = InProxySettings.bGenerateLightmapUVs;
		SrcModel.BuildSettings.bBuildReversedIndexBuffer = false;
		if (!InProxySettings.bAllowDistanceField)
		{
			SrcModel.BuildSettings.DistanceFieldResolutionScale = 0.0f;
		}

		ImposterStaticMesh->CreateMeshDescription(0, SourceMeshDesc);

		// Disable collisions on imposters
		FMeshSectionInfo Info = ImposterStaticMesh->GetSectionInfoMap().Get(0, 0);
		Info.bEnableCollision = false;
		ImposterStaticMesh->GetSectionInfoMap().Set(0, 0, Info);

		UMaterialInterface* ImposterMaterial = GetImposterMaterial(InComponent);

		ImposterStaticMesh->SetStaticMaterials({ ImposterMaterial });

		bMeshChanged = true;
	}
	else
	{
		// Compare existing Imposter mesh with the source mesh, update imposter if required
		FMeshDescription* ImposterMeshDesc = ImposterStaticMesh->GetMeshDescription(0);
		check(ImposterMeshDesc);

		FSHAHash ImposterHash = FStaticMeshOperations::ComputeSHAHash(*ImposterMeshDesc, true);
		FSHAHash SourceHash = FStaticMeshOperations::ComputeSHAHash(SourceMeshDesc, true);
		
		if (ImposterHash != SourceHash)
		{
			*ImposterMeshDesc = SourceMeshDesc;
			bMeshChanged = true;
		}

		// Validate source bounds extensions haven't changed
		if (!bMeshChanged && SourceHasBoundsExtension)
		{
			bMeshChanged = !SourcePositiveBoundsExtension.Equals(ImposterStaticMesh->GetNegativeBoundsExtension()) ||
						   !SourceNegativeBoundsExtension.Equals(ImposterStaticMesh->GetPositiveBoundsExtension());
		}
	}

	if (bMeshChanged)
	{
		// Commit mesh description and materials list to static mesh
		ImposterStaticMesh->CommitMeshDescription(0);

		//Set the Imported version before calling the build
		ImposterStaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

		ImposterStaticMesh->PostEditChange();

		// If the source has source bounds extensions, apply them unchanged
		if (SourceHasBoundsExtension)
		{
			ImposterStaticMesh->SetPositiveBoundsExtension(SourceImposterStaticMesh->GetPositiveBoundsExtension());
			ImposterStaticMesh->SetNegativeBoundsExtension(SourceImposterStaticMesh->GetNegativeBoundsExtension());
		}
		else
		{
			// Our imposters meshes are flat, but they actually represent a volume.
			// Extend the imposter bounds using the original mesh bounds.
			if (ImposterStaticMesh->GetBoundingBox().GetVolume() == 0)
			{
				const FBox StaticMeshBox = ImposterStaticMesh->GetBoundingBox();
				const FBox CombinedBox = StaticMeshBox + SourceImposterStaticMesh->GetBoundingBox();
				ImposterStaticMesh->SetPositiveBoundsExtension((CombinedBox.Max - StaticMeshBox.Max));
				ImposterStaticMesh->SetNegativeBoundsExtension((StaticMeshBox.Min - CombinedBox.Min));
			}
		}

		ImposterStaticMesh->CalculateExtendedBounds();

		ImposterStaticMesh->MarkPackageDirty();
	}

	return ImposterStaticMesh;
}


struct FHLODBuildParams
{
	ALODActor*								LODActor;
	UHLODProxy*								Proxy;
	const TArray<UPrimitiveComponent*>&		Components;
	FHierarchicalSimplification				LODSetup;
	UMaterialInterface*						BaseMaterial;
	UPackage*								AssetsOuter;
	FString									PackageName;

	FHLODBuildParams(const TArray<UPrimitiveComponent*>& InComponents, const FHierarchicalSimplification& InLODSetup)
		: Components(InComponents)
		, LODSetup(InLODSetup)
	{
		FMaterialProxySettings* MaterialProxySettings = LODSetup.GetSimplificationMethodMaterialSettings();
		if (MaterialProxySettings)
		{
			MaterialProxySettings->ResolveTexelDensity(InComponents);
		}
		else
		{
			UE_LOGF(LogHierarchicalLODUtilities, Error, "Unsupported simplification method provided");
		}
	}
};

struct FHLODBuildResults
{
	bool			bDeferredResults = false;
	UStaticMesh*	HLODMesh = nullptr;
	FVector			HLODLocation = FVector::ZeroVector;
};

FHLODBuildResults GenerateHLODMesh_Simplify(const FHLODBuildParams& InBuildParams)
{
	FHLODBuildResults HLODBuildResults;

	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();

	// Generate proxy mesh and proxy material assets
	IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	const bool bHasMeshReductionCapableModule = (MeshReductionModule.GetMeshMergingInterface() != nullptr);

	if (!bHasMeshReductionCapableModule)
	{
		return HLODBuildResults;
	}

	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	FHierarchicalLODProxyProcessor* Processor = Module.GetProxyProcessor();

	FHierarchicalSimplification OverrideLODSetup = InBuildParams.LODSetup;

	FMeshProxySettings ProxySettings = InBuildParams.LODSetup.ProxySetting;
	if (InBuildParams.LODActor->bOverrideMaterialMergeSettings)
	{
		ProxySettings.MaterialSettings = InBuildParams.LODActor->MaterialSettings;
	}

	if (InBuildParams.LODActor->bOverrideScreenSize)
	{
		ProxySettings.ScreenSize = InBuildParams.LODActor->ScreenSize;
	}

	if (InBuildParams.LODActor->bOverrideTransitionScreenSize)
	{
		OverrideLODSetup.TransitionScreenSize = InBuildParams.LODActor->TransitionScreenSize;
	}

	FGuid JobID = Processor->AddProxyJob(InBuildParams.LODActor, InBuildParams.Proxy, OverrideLODSetup);

	TArray<UStaticMeshComponent*> StaticMeshComponents;
	Algo::Transform(InBuildParams.Components, StaticMeshComponents, [](UPrimitiveComponent* InPrimitiveComponent) { return Cast<UStaticMeshComponent>(InPrimitiveComponent); });

	MeshMergeUtilities.CreateProxyMesh(StaticMeshComponents, ProxySettings, InBuildParams.BaseMaterial, InBuildParams.AssetsOuter, InBuildParams.PackageName, JobID, Processor->GetCallbackDelegate(), true, OverrideLODSetup.TransitionScreenSize);

	
	HLODBuildResults.bDeferredResults = true;
	
	return HLODBuildResults;
}

FHLODBuildResults GenerateHLODMesh_Merge(const FHLODBuildParams& InBuildParams)
{
	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();

	FMeshMergingSettings MergeSettings = InBuildParams.LODSetup.MergeSetting;
	if (InBuildParams.LODActor->bOverrideMaterialMergeSettings)
	{
		MergeSettings.MaterialSettings = InBuildParams.LODActor->MaterialSettings;
	}

	// update LOD parents before rebuild to ensure they are valid when mesh merge extensions are called.
	InBuildParams.LODActor->UpdateSubActorLODParents();

	FHLODBuildResults HLODBuildResults;

	TArray<UObject*> OutAssets;
	MeshMergeUtilities.MergeComponentsToStaticMesh(InBuildParams.Components, InBuildParams.LODActor->GetWorld(), MergeSettings, InBuildParams.BaseMaterial, InBuildParams.AssetsOuter, InBuildParams.PackageName, OutAssets, HLODBuildResults.HLODLocation, InBuildParams.LODSetup.TransitionScreenSize, true);

	// set staticmesh
	for (UObject* Asset : OutAssets)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
		{
			HLODBuildResults.HLODMesh = StaticMesh;
			break;
		}
	}

	return HLODBuildResults;
}

FHLODBuildResults GenerateHLODMesh_Approximate(const FHLODBuildParams& InBuildParams)
{
	FHLODBuildResults HLODBuildResults;

	IGeometryProcessingInterfacesModule* GeomProcInterfaces = FModuleManager::Get().LoadModulePtr<IGeometryProcessingInterfacesModule>("GeometryProcessingInterfaces");
	IGeometryProcessing_ApproximateActors* ApproxActorsAPI = GeomProcInterfaces ? GeomProcInterfaces->GetApproximateActorsImplementation() : nullptr;
	const bool bHasApproximateActorsModule = ApproxActorsAPI != nullptr;

	if (!ApproxActorsAPI)
	{
		return HLODBuildResults;
	}

	const FMeshApproximationSettings& UseSettings = InBuildParams.LODSetup.ApproximateSettings;
	UMaterialInterface* HLODMaterial = InBuildParams.BaseMaterial;

	IGeometryProcessing_ApproximateActors::FOptions Options = ApproxActorsAPI->ConstructOptions(UseSettings);
	Options.bGenerateLightmapUVs = false;
	Options.bCreatePhysicsBody = false;

	// Material baking settings
	Options.BakeMaterial = HLODMaterial;
	if (!FMaterialUtilities::IsValidFlattenMaterial(Options.BakeMaterial))
	{
		Options.BakeMaterial = GEngine->DefaultFlattenMaterial;
	}
	Options.BaseColorTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Diffuse, Options.BakeMaterial));
	Options.NormalTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Normal, Options.BakeMaterial));
	Options.MetallicTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Metallic, Options.BakeMaterial));
	Options.RoughnessTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Roughness, Options.BakeMaterial));
	Options.SpecularTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Specular, Options.BakeMaterial));
	Options.EmissiveTexParamName = FName("EmissiveHDRTexture"); // TODO - Approximate actors should look up if the material sampler is expecting an HDR texture and capture accordingly
	Options.bUsePackedMRS = true;
	Options.PackedMRSTexParamName = FName("PackedTexture");

	// Use temp packages - otherwise Approximate Actors will create it's mesh using the same name as the HLOD Proxy object.
	const FString NewAssetNamePrefix(TEXT("NEWASSET_"));
	FString PackageName = InBuildParams.AssetsOuter->GetPackage()->GetName();
	FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
	FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);
	Options.BasePackagePath = PackagePath / NewAssetNamePrefix + AssetName;

	// run actor approximation computation
	IGeometryProcessing_ApproximateActors::FInput Input;
	Algo::Transform(InBuildParams.Components, Input.Components, [](UPrimitiveComponent* PrimitiveComponent) { return PrimitiveComponent; });

	IGeometryProcessing_ApproximateActors::FResults Results;
	ApproxActorsAPI->ApproximateActors(Input, Options, Results);

	auto RenameNewAsset = [&PackagePath, &NewAssetNamePrefix, &InBuildParams](UObject* NewAsset)
	{
		FString AssetName = NewAsset->GetName();
		AssetName.RemoveFromStart(NewAssetNamePrefix);
			
		// Add a prefix to the generated static mesh, otherwise it would share the same name as the HLOD Proxy object.
		if (NewAsset->IsA<UStaticMesh>())
		{
			AssetName += TEXT("_Mesh");
		}

		// Remplace existing asset
		UObject* AssetToReplace = StaticFindObjectFast(UObject::StaticClass(), InBuildParams.AssetsOuter, *AssetName);
		if (AssetToReplace)
		{
			// Replace references
			TArray<UObject*> ObjectsToReplace(&AssetToReplace, 1);
			ObjectTools::ForceReplaceReferences(NewAsset, ObjectsToReplace);

			// Move the previous asset to the transient package
			AssetToReplace->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
		}
			
		NewAsset->Rename(*AssetName, InBuildParams.AssetsOuter, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	};

	Algo::ForEach(Results.NewMeshAssets, RenameNewAsset);
	Algo::ForEach(Results.NewMaterials, RenameNewAsset);
	Algo::ForEach(Results.NewTextures, RenameNewAsset);
	
	if (Results.ResultCode == IGeometryProcessing_ApproximateActors::EResultCode::Success)
	{
		// Retrieve staticmesh
		if (ensure(Results.NewMeshAssets.Num() == 1))
		{
			HLODBuildResults.HLODMesh = Results.NewMeshAssets[0];
		}

		// Setup material switches
		for (UMaterialInterface* Material : Results.NewMaterials)
		{
			UMaterialInstance* MaterialInst = CastChecked<UMaterialInstance>(Material);

			FStaticParameterSet StaticParameterSet;

			auto SetStaticSwitch = [&StaticParameterSet](FName ParamName, bool bSet)
			{
				if (bSet)
				{
					FStaticSwitchParameter SwitchParameter;
					SwitchParameter.ParameterInfo.Name = ParamName;
					SwitchParameter.Value = true;
					SwitchParameter.bOverride = true;
					StaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);
				}
			};

			// Set proper switches needed by our base flatten material
			SetStaticSwitch("UseBaseColor", Options.bBakeBaseColor);
			SetStaticSwitch("UseDiffuse", Options.bBakeBaseColor);
			SetStaticSwitch("UseRoughness", Options.bBakeRoughness);
			SetStaticSwitch("UseMetallic", Options.bBakeMetallic);
			SetStaticSwitch("UseSpecular", Options.bBakeSpecular);
			SetStaticSwitch("UseEmissive", Options.bBakeEmissive);
			SetStaticSwitch("UseEmissiveColor", Options.bBakeEmissive);
			SetStaticSwitch("UseEmissiveHDR", Options.bBakeEmissive);
			SetStaticSwitch("UseNormal", Options.bBakeNormalMap);
			SetStaticSwitch("PackMetallic", Options.bUsePackedMRS);
			SetStaticSwitch("PackSpecular", Options.bUsePackedMRS);
			SetStaticSwitch("PackRoughness", Options.bUsePackedMRS);

			// Force initializing the static permutations according to the switches we have set
			MaterialInst->UpdateStaticPermutation(StaticParameterSet);
			MaterialInst->InitStaticPermutation();
			MaterialInst->PostEditChange();
		}
	}

	return HLODBuildResults;
}


bool FHierarchicalLODUtilities::BuildStaticMeshForLODActor(ALODActor* LODActor, UHLODProxy* Proxy, const FHierarchicalSimplification& LODSetup, UMaterialInterface* InBaseMaterial)
{
	if (!Proxy || !LODActor)
	{	
		return false;
	}

	UE_LOGF(LogHierarchicalLODUtilities, Log, "Building Proxy Mesh for Cluster %ls", *LODActor->GetName());
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_BuildProxyMesh", "Building Proxy Mesh for Cluster"));

	// Pass false here and dirty package later if values have changed
	LODActor->Modify(false);
	Proxy->Modify();

	// Clean out the proxy as we are rebuilding meshes
	Proxy->Clean();
	UPackage* AssetsOuter = Proxy->GetOutermost();

	TArray<UPrimitiveComponent*> AllComponents;
	UHLODProxy::ExtractComponents(LODActor, AllComponents);

	// It shouldn't even have come here if it didn't have any static meshes
	if(!ensure(AllComponents.Num() > 0))
	{
		return false;
	}

	TArray<UStaticMeshComponent*> AllInstances;
	if (LODSetup.MergeSetting.bIncludeImposters)
	{			
		// Retrieve all imposters.
		for (UPrimitiveComponent* Component : AllComponents)
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
			{
				if (LODActor->ShouldUseInstancing(StaticMeshComponent))
				{
					AllInstances.Add(StaticMeshComponent);
				}
			}
		}

		// Instances won't be merged in the HLOD mesh
		AllComponents.RemoveAll([&](UPrimitiveComponent* Component) { return AllInstances.Contains(Component); });
	}

	if (AllComponents.Num() > 0)
	{
		FHLODBuildParams HLODBuildParams(AllComponents, LODSetup);
		HLODBuildParams.LODActor = LODActor;
		HLODBuildParams.Proxy = Proxy;
		HLODBuildParams.BaseMaterial = InBaseMaterial;
		HLODBuildParams.AssetsOuter = AssetsOuter;

		// Should give a unique name, so use the LODActor tag, or if empty, the first actor name
		FString LODActorTag = LODActor->GetLODActorTag();
		if (LODActorTag.IsEmpty())
		{
			const AActor* FirstActor = UHLODProxy::FindFirstActor(LODActor);
			LODActorTag = *FirstActor->GetName();
		}
		HLODBuildParams.PackageName = FString::Printf(TEXT("%s_%i_%s"), *(AssetsOuter->GetName()), LODActor->LODLevel - 1, *LODActorTag);

		FHLODBuildResults HLODBuildResults;
		switch (LODSetup.SimplificationMethod)
		{
		case EHierarchicalSimplificationMethod::Merge:
			HLODBuildResults = GenerateHLODMesh_Merge(HLODBuildParams);
			break;
		case EHierarchicalSimplificationMethod::Simplify:
			HLODBuildResults = GenerateHLODMesh_Simplify(HLODBuildParams);
			break;
		case EHierarchicalSimplificationMethod::Approximate:
			HLODBuildResults = GenerateHLODMesh_Approximate(HLODBuildParams);
			break;

		default:
			UE_LOGF(LogHierarchicalLODUtilities, Error, "Unsupported simplification method provided");
			return false;
		}
		
		if (!HLODBuildResults.bDeferredResults)
		{
			if (!HLODBuildResults.HLODMesh)
			{
				UE_LOGF(LogHierarchicalLODUtilities, Warning, "No HLOD mesh generated");
				return false;
			}

			UStaticMesh* MainMesh = HLODBuildResults.HLODMesh;

			// make sure the mesh won't affect navmesh generation
			MainMesh->MarkAsNotHavingNavigationData();

			bool bDirtyPackage = false;
			UStaticMesh* PreviousStaticMesh = LODActor->GetStaticMeshComponent()->GetStaticMesh();
			bDirtyPackage |= (MainMesh != PreviousStaticMesh);
			LODActor->SetStaticMesh(MainMesh);
			bDirtyPackage |= (LODActor->GetActorLocation() != HLODBuildResults.HLODLocation);
			LODActor->SetActorLocation(HLODBuildResults.HLODLocation);

			// Check resulting mesh and give a warning if it exceeds the vertex / triangle cap for certain platforms
			FProjectStatus ProjectStatus;
			if (IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && (ProjectStatus.IsTargetPlatformSupported(TEXT("Android")) || ProjectStatus.IsTargetPlatformSupported(TEXT("IOS"))))
			{
				if (MainMesh->GetRenderData() && MainMesh->GetRenderData()->LODResources.Num() && MainMesh->GetRenderData()->LODResources[0].IndexBuffer.Is32Bit())
				{
					FMessageLog("HLODResults").Warning()
						->AddToken(FUObjectToken::Create(LODActor))
						->AddToken(FTextToken::Create(LOCTEXT("HLODError_MeshNotBuildTwo", " Mesh has more that 65535 vertices, incompatible with mobile; forcing 16-bit (will probably cause rendering issues).")));
				}
			}

			// At the moment this assumes a fixed field of view of 90 degrees (horizontal and vertical axi)
			static const float FOVRad = 90.0f * (float)PI / 360.0f;
			static const FMatrix ProjectionMatrix = FPerspectiveMatrix(FOVRad, 1920, 1080, 0.01f);
			FBoxSphereBounds Bounds = LODActor->GetStaticMeshComponent()->CalcBounds(FTransform());

			float DrawDistance;
			if (LODSetup.bUseOverrideDrawDistance)
			{
				DrawDistance = LODSetup.OverrideDrawDistance;
			}
			else
			{
				DrawDistance = CalculateDrawDistanceFromScreenSize(Bounds.SphereRadius, LODSetup.TransitionScreenSize, ProjectionMatrix);
			}

			bDirtyPackage |= (LODActor->GetDrawDistance() != DrawDistance);
			LODActor->SetDrawDistance(DrawDistance);

			bDirtyPackage |= (LODActor->GetStaticMeshComponent()->bRayTracingFarField != true);
			LODActor->GetStaticMeshComponent()->bRayTracingFarField = true;

			LODActor->DetermineShadowingFlags();

			// Link proxy to actor
			const UHLODProxy* PreviousProxy = LODActor->GetProxy();
			Proxy->AddMesh(LODActor, MainMesh, UHLODProxy::GenerateKeyForActor(LODActor));
			bDirtyPackage |= (LODActor->GetProxy() != PreviousProxy);

			if(bDirtyPackage && !LODActor->WasBuiltFromHLODDesc())
			{
				LODActor->MarkPackageDirty();
			}

			// Clean out standalone meshes from the proxy package as we are about to GC, and mesh merging creates assets that are 
			// supposed to be standalone
			CleanStandaloneAssetsInPackage(AssetsOuter);

			// Collect garbage to clean up old unreferenced data in the HLOD package
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}

	// Add imposters
	LODActor->ClearInstances();
	if (AllInstances.Num() > 0)
	{
		struct FLODInstanceBatch
		{
			UStaticMesh*					StaticMesh;
			TArray<FTransform>				Transforms;
			TArray<FCustomPrimitiveData>	CustomPrimitiveData;
		};

		// Get all meshes + transforms for all instances type (per material)
		TMap<FHLODInstancingKey, FLODInstanceBatch> InstancesBatches;
		for (UStaticMeshComponent* SMC : AllInstances)
		{
			UStaticMesh* StaticMesh = SMC->GetStaticMesh();
			check(StaticMesh);

			UMaterialInterface* InstanceMaterial = GetImposterMaterial(SMC);

			FHLODInstancingKey Key(SMC->GetStaticMesh(), InstanceMaterial);
			check(Key.IsValid());

			FLODInstanceBatch& LODInstanceBatch = InstancesBatches.FindOrAdd(Key);

			FTransform ComponentTransformWS = SMC->GetComponentTransform();

			// If we have an ISMC, ensure we include all its instances
			if (UInstancedStaticMeshComponent* InstancedSMC = Cast<UInstancedStaticMeshComponent>(SMC))
			{
				LODInstanceBatch.Transforms.Reserve(LODInstanceBatch.Transforms.Num() + InstancedSMC->GetInstanceCount());
				for (const FInstancedStaticMeshInstanceData& InstanceData : InstancedSMC->PerInstanceSMData)
				{
					FTransform InstanceTransformWS = FTransform(InstanceData.Transform) * ComponentTransformWS;
					LODInstanceBatch.Transforms.Add(InstanceTransformWS);
				}
			}
			else
			{
				LODInstanceBatch.Transforms.Add(ComponentTransformWS);
				LODInstanceBatch.CustomPrimitiveData.Add(SMC->GetCustomPrimitiveData());
			}

			// The static mesh hasn't been created yet, do it.
			if (LODInstanceBatch.StaticMesh == nullptr)
			{
				LODInstanceBatch.StaticMesh = CreateImposterStaticMesh(SMC, LODSetup.ProxySetting);
			}
		}

		// Add imposters to the LODActor
		for (const auto& ImposterBatch : InstancesBatches)
		{
			LODActor->AddInstances(ImposterBatch.Value.StaticMesh, ImposterBatch.Key.Material, ImposterBatch.Value.Transforms, ImposterBatch.Value.CustomPrimitiveData);
		}
	}

	return true;
}

EClusterGenerationError FHierarchicalLODUtilities::ShouldGenerateCluster(AActor* Actor, const int32 HLODLevelIndex)
{
	if (!Actor)
	{
		return EClusterGenerationError::InvalidActor;
	}

	if (Actor->IsHidden())
	{
		return EClusterGenerationError::ActorHiddenInGame;
	}

	if (!Actor->bEnableAutoLODGeneration)
	{
		return EClusterGenerationError::ExcludedActor;
	}

	ALODActor* LODActor = Cast<ALODActor>(Actor);
	if (LODActor)
	{
		return EClusterGenerationError::LODActor;
	}

	FVector Origin, Extent;
	Actor->GetActorBounds(false, Origin, Extent);
	if (Extent.SizeSquared() <= 0.1)
	{
		return EClusterGenerationError::ActorTooSmall;
	}

	// for now only consider staticmesh - I don't think skel mesh would work with simplygon merge right now @fixme
	TArray<UStaticMeshComponent*> Components;
	Actor->GetComponents(Components);

	int32 ValidComponentCount = 0;
	// now make sure you check parent primitive, so that we don't build for the actor that already has built. 

	EClusterGenerationError ErrorType = EClusterGenerationError::None;

	if (Components.Num() > 0)
	{
		for (UStaticMeshComponent* ComponentIter : Components)
		{
			if (ComponentIter->GetLODParentPrimitive())
			{
				return EClusterGenerationError::AlreadyClustered;
			}

			if (ComponentIter->bHiddenInGame)
			{
				return EClusterGenerationError::ComponentHiddenInGame;
			}

			// see if we should generate it
			if (ComponentIter->ShouldGenerateAutoLOD(HLODLevelIndex))
			{
				++ValidComponentCount;
				ErrorType |= EClusterGenerationError::ValidActor;
			}
			else
			{
				ErrorType |= (ComponentIter->bEnableAutoLODGeneration ? EClusterGenerationError::MoveableComponent: EClusterGenerationError::ExcludedComponent);
			}
		}
	}

	return ErrorType;
}

ALODActor* FHierarchicalLODUtilities::GetParentLODActor(const AActor* InActor)
{
	if (InActor)
	{
		TArray<UStaticMeshComponent*> ComponentArray;
		InActor->GetComponents(ComponentArray);
		for (auto Component : ComponentArray)
		{
			UPrimitiveComponent* ParentComponent = Component->GetLODParentPrimitive();
			if (ParentComponent)
			{
				if (ALODActor* ParentActor = Cast<ALODActor>(ParentComponent->GetOwner()))
				{
					return ParentActor;
				}
			}
		}
	}

	return nullptr;
}

void FHierarchicalLODUtilities::DestroyCluster(ALODActor* InActor)
{
	// Find if it has a parent ALODActor
	AActor* Actor = InActor;
	UWorld* World = Actor->GetWorld();
	ALODActor* ParentLOD = GetParentLODActor(InActor);

	// Only dirty the level if LODActors weren't spawned from an HLOD desc
	bool bShouldDirtyLevel = !InActor->WasBuiltFromHLODDesc();

	const FScopedTransaction Transaction(LOCTEXT("UndoAction_DeleteCluster", "Deleting a (invalid) Cluster"));
	Actor->Modify(bShouldDirtyLevel);

	UHLODProxy* HLODProxy = InActor->GetProxy();

	if (ParentLOD != nullptr)
	{
		ParentLOD->Modify(bShouldDirtyLevel);
		ParentLOD->RemoveSubActor(Actor);
	}

	// Clean out sub actors and update their LODParent
	while (InActor->SubActors.Num())
	{
		AActor* SubActor = InActor->SubActors[0];
		SubActor->Modify(bShouldDirtyLevel);
		InActor->RemoveSubActor(SubActor);
	}

	World->DestroyActor(InActor, false);

	if (ParentLOD != nullptr && !ParentLOD->HasAnySubActors())
	{
		DestroyCluster(ParentLOD);
	}
}

ALODActor* FHierarchicalLODUtilities::CreateNewClusterActor(UWorld* InWorld, const int32 InLODLevel, AWorldSettings* WorldSettings)
{
	// Check incoming data
	check(InWorld != nullptr && WorldSettings != nullptr && InLODLevel >= 0);
	const TArray<struct FHierarchicalSimplification>& HierarchicalLODSetups = InWorld->GetWorldSettings()->GetHierarchicalLODSetup();
	if (HierarchicalLODSetups.Num() == 0 || HierarchicalLODSetups.Num() < InLODLevel)
	{
		return nullptr;
	}

	// LODActors that are saved to HLOD packages must be transient
	FActorSpawnParameters ActorSpawnParams;
	ActorSpawnParams.ObjectFlags = GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages ? EObjectFlags::RF_Transient | EObjectFlags::RF_DuplicateTransient : EObjectFlags::RF_NoFlags;

	// Spawn and setup actor
	ALODActor* NewActor = InWorld->SpawnActor<ALODActor>(ALODActor::StaticClass(), ActorSpawnParams);
	NewActor->LODLevel = InLODLevel + 1;
	NewActor->CachedNumHLODLevels = WorldSettings->GetNumHierarchicalLODLevels();
	NewActor->SetDrawDistance(0.0f);
	NewActor->SetStaticMesh(nullptr);
	NewActor->PostEditChange();

	return NewActor;
}

ALODActor* FHierarchicalLODUtilities::CreateNewClusterFromActors(UWorld* InWorld, AWorldSettings* WorldSettings, const TArray<AActor*>& InActors, const int32 InLODLevel /*= 0*/)
{
	checkf(InWorld != nullptr, TEXT("Invalid world"));
	checkf(InActors.Num() > 0, TEXT("Zero number of sub actors"));
	checkf(WorldSettings != nullptr, TEXT("Invalid world settings"));

	const bool bWasWorldPackageDirty = InWorld->GetOutermost()->IsDirty();

	const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateNewCluster", "Create new Cluster"));
	InWorld->Modify(false);

	// Create the cluster
	ALODActor* NewCluster = CreateNewClusterActor(InWorld, InLODLevel, WorldSettings);
	checkf(NewCluster != nullptr, TEXT("Failed to create a new cluster"));

	// Add InActors to new cluster
	for (AActor* Actor : InActors)
	{
		checkf(Actor != nullptr, TEXT("Invalid actor in InActors"));
		
		// Check if Actor is currently part of a different cluster
		ALODActor* ParentActor = GetParentLODActor(Actor);
		if (ParentActor != nullptr)
		{
			// If so remove it first
			ParentActor->Modify();
			ParentActor->RemoveSubActor(Actor);

			// If the parent cluster is now empty (invalid) destroy it
			if (!ParentActor->HasAnySubActors())
			{
				DestroyCluster(ParentActor);
			}
		}

		// Add actor to new cluster
		NewCluster->AddSubActor(Actor);
	}

	// Update sub actor LOD parents to populate 
	NewCluster->UpdateSubActorLODParents();

	if (GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages)
	{
		UHLODProxy* Proxy = CreateOrRetrieveLevelHLODProxy(InWorld->PersistentLevel, NewCluster->LODLevel - 1);
		Proxy->AddLODActor(NewCluster);

		// Don't dirty the level file after spawning a transient actor
		if (!bWasWorldPackageDirty)
		{
			InWorld->GetOutermost()->SetDirtyFlag(false);
		}
	}
	else
	{
		NewCluster->MarkPackageDirty();
	}	

	return NewCluster;
}

const bool FHierarchicalLODUtilities::RemoveActorFromCluster(AActor* InActor)
{
	checkf(InActor != nullptr, TEXT("Invalid InActor"));
	
	bool bSucces = false;

	ALODActor* ParentActor = GetParentLODActor(InActor);
	if (ParentActor != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("UndoAction_RemoveActorFromCluster", "Remove Actor From Cluster"));
		ParentActor->Modify();
		InActor->Modify();

		bSucces = ParentActor->RemoveSubActor(InActor);

		if (!ParentActor->HasAnySubActors())
		{
			DestroyCluster(ParentActor);
		}
	}
	
	return bSucces;
}

const bool FHierarchicalLODUtilities::AddActorToCluster(AActor* InActor, ALODActor* InParentActor)
{
	checkf(InActor != nullptr, TEXT("Invalid InActor"));
	checkf(InParentActor != nullptr, TEXT("Invalid InParentActor"));

	// First, if it is the case remove the actor from it's current cluster
	const bool bActorWasClustered = RemoveActorFromCluster(InActor);

	// Now add it to the new one
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_AddActorToCluster", "Add Actor To Cluster"));
	InParentActor->Modify();
	InActor->Modify();

	// Add InActor to InParentActor cluster
	InParentActor->AddSubActor(InActor);

#if WITH_EDITOR
	GEditor->BroadcastHLODActorAdded(InActor, InParentActor);
#endif // WITH_EDITOR

	return true;
}

const bool FHierarchicalLODUtilities::MergeClusters(ALODActor* TargetCluster, ALODActor* SourceCluster)
{
	checkf(TargetCluster != nullptr&& TargetCluster->SubActors.Num() > 0, TEXT("Invalid InActor"));
	checkf(SourceCluster != nullptr && SourceCluster->SubActors.Num() > 0, TEXT("Invalid InParentActor"));

	const FScopedTransaction Transaction(LOCTEXT("UndoAction_MergeClusters", "Merge Clusters"));
	TargetCluster->Modify();
	SourceCluster->Modify();

	while (SourceCluster->SubActors.Num())
	{
		AActor* SubActor = SourceCluster->SubActors.Last();
		AddActorToCluster(SubActor, TargetCluster);		
	}

	if (!SourceCluster->HasAnySubActors())
	{
		DestroyCluster(SourceCluster);
	}

	return true;
}

const bool FHierarchicalLODUtilities::AreActorsInSamePersistingLevel(const TArray<AActor*>& InActors)
{
	ULevel* Level = nullptr;
	bool bSameLevelInstance = true;
	for (AActor* Actor : InActors)
	{
		if (Level == nullptr)
		{
			Level = Actor->GetLevel();
		}

		bSameLevelInstance &= (Level == Actor->GetLevel());

		if (!bSameLevelInstance)
		{
			break;
		}
	}

	return bSameLevelInstance;
}

const bool FHierarchicalLODUtilities::AreClustersInSameHLODLevel(const TArray<ALODActor*>& InLODActors)
{
	int32 HLODLevel = -1;
	bool bSameHLODLevel = true;
	for (ALODActor* LODActor : InLODActors)
	{
		if (HLODLevel == -1)
		{
			HLODLevel = LODActor->LODLevel;
		}

		bSameHLODLevel &= (HLODLevel == LODActor->LODLevel);

		if (!bSameHLODLevel)
		{
			break;
		}
	}

	return bSameHLODLevel;
}

const bool FHierarchicalLODUtilities::AreActorsInSameHLODLevel(const TArray<AActor*>& InActors)
{
	int32 HLODLevel = -1;
	bool bSameHLODLevel = true;
	for (AActor* Actor : InActors)
	{
		ALODActor* ParentActor = FHierarchicalLODUtilities::GetParentLODActor(Actor);

		if (ParentActor != nullptr)
		{
			if (HLODLevel == -1)
			{
				HLODLevel = ParentActor->LODLevel;
			}

			bSameHLODLevel &= (HLODLevel == ParentActor->LODLevel);
		}
		else
		{
			bSameHLODLevel = false;
		}

		if (!bSameHLODLevel)
		{
			break;
		}
	}

	return bSameHLODLevel;
}

const bool FHierarchicalLODUtilities::AreActorsClustered(const TArray<AActor*>& InActors)
{	
	bool bClustered = true;
	for (AActor* Actor : InActors)
	{
		bClustered &= (GetParentLODActor(Actor) != nullptr);

		if (!bClustered)
		{
			break;
		}
	}

	return bClustered;
}

const bool FHierarchicalLODUtilities::IsActorClustered(const AActor* InActor)
{
	bool bClustered = (GetParentLODActor(InActor) != nullptr);	
	return bClustered;
}

void FHierarchicalLODUtilities::ExcludeActorFromClusterGeneration(AActor* InActor)
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_ExcludeActorFromClusterGeneration", "Exclude Actor From Cluster Generation"));
	InActor->Modify();
	InActor->bEnableAutoLODGeneration = false;
	RemoveActorFromCluster(InActor);
}

void FHierarchicalLODUtilities::DestroyLODActor(ALODActor* InActor)
{
	DestroyCluster(InActor);
}

void FHierarchicalLODUtilities::ExtractStaticMeshActorsFromLODActor(ALODActor* LODActor, TArray<AActor*> &InOutActors)
{
	for (auto ChildActor : LODActor->SubActors)
	{
		if (ChildActor)
		{
			TArray<AActor*> ChildActors;
			if (ChildActor->IsA<ALODActor>())
			{
				ExtractStaticMeshActorsFromLODActor(Cast<ALODActor>(ChildActor), ChildActors);
			}

			ChildActors.Push(ChildActor);
			InOutActors.Append(ChildActors);
		}
	}	
}

void FHierarchicalLODUtilities::DeleteLODActorsInHLODLevel(UWorld* InWorld, const int32 HLODLevelIndex)
{
	// you still have to delete all objects just in case they had it and didn't want it anymore
	TArray<UObject*> AssetsToDelete;
	for (int32 ActorId = InWorld->PersistentLevel->Actors.Num() - 1; ActorId >= 0; --ActorId)
	{
		ALODActor* LodActor = Cast<ALODActor>(InWorld->PersistentLevel->Actors[ActorId]);
		if (LodActor && LodActor->LODLevel == (HLODLevelIndex + 1))
		{
			DestroyCluster(LodActor);
		}
	}
}

int32 FHierarchicalLODUtilities::ComputeStaticMeshLODLevel(const TArray<FStaticMeshSourceModel>& SourceModels, const FStaticMeshRenderData* RenderData, const float ScreenSize)
{
	const int32 NumLODs = !SourceModels.IsEmpty() ? SourceModels.Num() : RenderData->LODResources.Num();

	// Walk backwards and return the first matching LOD
	for (int32 LODIndex = NumLODs - 1; LODIndex >= 0; --LODIndex)
	{
		const FPerPlatformFloat& LODSceenSize = SourceModels.IsValidIndex(LODIndex) && SourceModels[LODIndex].ScreenSize.Default != 0.0f ? SourceModels[LODIndex].ScreenSize : RenderData->ScreenSize[LODIndex];

		// For HLOD generation we want the default values and not the per platform overrides
		if (LODSceenSize.Default > ScreenSize)
		{
			return FMath::Max(LODIndex, 0);
		}
	}

	return 0;
}

int32 FHierarchicalLODUtilities::GetLODLevelForScreenSize(const UStaticMeshComponent* StaticMeshComponent, const float ScreenSize)
{
	check(StaticMeshComponent != nullptr);

	const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	check(StaticMesh != nullptr);

	const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
	checkf(RenderData != nullptr, TEXT("StaticMesh %s referenced by StaticMeshComponent %s contains invalid render data"), *StaticMesh->GetName(), *StaticMeshComponent->GetName());

	return ComputeStaticMeshLODLevel(StaticMesh->GetSourceModels(), RenderData, ScreenSize);
}

AHierarchicalLODVolume* FHierarchicalLODUtilities::CreateVolumeForLODActor(ALODActor* InLODActor, UWorld* InWorld)
{
	FBox BoundingBox = InLODActor->GetComponentsBoundingBox(true);

	// If no world is provided, spawn the volume in the same level as InLODActor
	UWorld* WorldToSpawnIn = InWorld ? InWorld : InLODActor->GetWorld();
	FActorSpawnParameters ActorSpawnParameters;
	ActorSpawnParameters.OverrideLevel = InWorld == nullptr ? InLODActor->GetLevel() : nullptr;

	AHierarchicalLODVolume* Volume = WorldToSpawnIn->SpawnActor<AHierarchicalLODVolume>(AHierarchicalLODVolume::StaticClass(), FTransform(BoundingBox.GetCenter()), ActorSpawnParameters);

	// this code builds a brush for the new actor
	Volume->PreEditChange(NULL);

	Volume->PolyFlags = 0;
	Volume->Brush = NewObject<UModel>(Volume, NAME_None, RF_Transactional);
	Volume->Brush->Initialize(nullptr, true);
	Volume->Brush->Polys = NewObject<UPolys>(Volume->Brush, NAME_None, RF_Transactional);
	Volume->GetBrushComponent()->Brush = Volume->Brush;
	Volume->BrushBuilder = NewObject<UCubeBuilder>(Volume, NAME_None, RF_Transactional);

	UCubeBuilder* CubeBuilder = static_cast<UCubeBuilder*>(Volume->BrushBuilder);

	CubeBuilder->X = BoundingBox.GetSize().X * 1.5f;
	CubeBuilder->Y = BoundingBox.GetSize().Y * 1.5f;
	CubeBuilder->Z = BoundingBox.GetSize().Z * 1.5f;

	Volume->BrushBuilder->Build(WorldToSpawnIn, Volume);

	FBSPOps::csgPrepMovingBrush(Volume);

	// Set the texture on all polys to NULL.  This stops invisible textures
	// dependencies from being formed on volumes.
	if (Volume->Brush)
	{
		for (int32 poly = 0; poly < Volume->Brush->Polys->Element.Num(); ++poly)
		{
			FPoly* Poly = &(Volume->Brush->Polys->Element[poly]);
			Poly->Material = NULL;
		}
	}

	Volume->PostEditChange();

	return Volume;
}

void FHierarchicalLODUtilities::HandleActorModified(AActor* InActor)
{
	ALODActor* ParentActor = GetParentLODActor(InActor);

	if (ParentActor != nullptr )
	{
		// So something in the actor changed that require use to flag the cluster as dirty
		ParentActor->Modify();
	}
}

bool FHierarchicalLODUtilities::IsWorldUsedForStreaming(const UWorld* InWorld)
{
	// @todo: This function is preventing users from editing HLOD settings in maps that happen to be used by both streaming and non-streaming maps.
	// @todo: This function is very expensive and can be called every single frame from the HLOD Outliner delegates.  It's usage needs to be optimized before we can re-enable it.

#if 0
	// Find references to the given world's outer package
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetIdentifier> ReferenceNames;
	const UPackage* OuterPackage = InWorld->GetOutermost();
	AssetRegistryModule.Get().GetReferencers(FAssetIdentifier(OuterPackage->GetFName()), ReferenceNames);

	for (const FAssetIdentifier& Identifier : ReferenceNames)
	{
		// Referncers can include things like primary asset virtual packages, we don't want those
		if (Identifier.PackageName != NAME_None)
		{
			const FString PackageName = Identifier.PackageName.ToString();
			UPackage* ReferencingPackage = FindPackage(nullptr, *PackageName);
			if (!ReferencingPackage)
			{
				ReferencingPackage = LoadPackage(nullptr, *PackageName, LOAD_None);
			}

			// Retrieve the referencing UPackage and check if it contains a map asset
			if (ReferencingPackage && ReferencingPackage->ContainsMap())
			{
				TArray<UPackage*> Packages;
				Packages.Add(ReferencingPackage);
				TArray<UObject*> Objects;
				UPackageTools::GetObjectsInPackages(&Packages, Objects);

				// Loop over all objects in package and try to find a world
				for (UObject* Object : Objects)
				{
					if (UWorld* World = Cast<UWorld>(Object))
					{
						// Check the world contains InWorld as a streaming level
						if (World->GetStreamingLevels().FindByPredicate([InWorld](const ULevelStreaming* StreamingLevel)
						{
							return StreamingLevel->GetWorldAsset() == InWorld;
						}))
						{
							return true;
						}
					}
				}
			}
		}
	}
#endif

	return false;
}

#undef LOCTEXT_NAMESPACE
