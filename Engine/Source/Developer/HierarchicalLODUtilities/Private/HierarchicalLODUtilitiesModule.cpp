// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchicalLODUtilitiesModule.h"
#include "Modules/ModuleManager.h"
#include "HierarchicalLODUtilities.h"
#include "HierarchicalLODProxyProcessor.h"

IMPLEMENT_MODULE(FHierarchicalLODUtilitiesModule, HierarchicalLODUtilities);

void FHierarchicalLODUtilitiesModule::StartupModule()
{
	ProxyProcessor = nullptr;
	Utilities = nullptr;
}

void FHierarchicalLODUtilitiesModule::ShutdownModule()
{
	// Clean up proxy processor and utilities instances
	if (ProxyProcessor)
	{
		delete ProxyProcessor;
		ProxyProcessor = nullptr;
	}

	if (Utilities)
	{
		delete Utilities;
		Utilities = nullptr;
	}

	ProxyProcessor = nullptr;
}

FHierarchicalLODProxyProcessor* FHierarchicalLODUtilitiesModule::GetProxyProcessor()
{
	if (ProxyProcessor == nullptr)
	{
		ProxyProcessor = new FHierarchicalLODProxyProcessor();
	}

	return ProxyProcessor;
}

IHierarchicalLODUtilities* FHierarchicalLODUtilitiesModule::GetUtilities()
{
	if (Utilities == nullptr)
	{
		Utilities = new FHierarchicalLODUtilities();
	}

	return Utilities;
}
