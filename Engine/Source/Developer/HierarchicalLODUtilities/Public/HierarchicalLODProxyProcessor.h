// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Containers/Ticker.h"
#include "HLOD/HLODSetup.h"
#include "MeshUtilities.h"
#include "IMeshReductionInterfaces.h"
#include "UObject/StrongObjectPtr.h"

class ALODActor;
class UHLODProxy;

class FHierarchicalLODProxyProcessor : public FTSTickerObjectBase
{
public:		
	HIERARCHICALLODUTILITIES_API FHierarchicalLODProxyProcessor();
	HIERARCHICALLODUTILITIES_API ~FHierarchicalLODProxyProcessor();
	
	/** Begin FTSTickerObjectBase */
	HIERARCHICALLODUTILITIES_API virtual bool Tick(float DeltaTime) override;
	/** End FTSTickerObjectBase */

	/**
	* AddProxyJob
	* @param InLODActor - LODActor for which the mesh will be generated
	* @param InProxy - The proxy mesh used to store the mesh
	* @param LODSetup - Simplification settings structure
	* @return FGuid - Guid for the job
	*/
	HIERARCHICALLODUTILITIES_API FGuid AddProxyJob(ALODActor* InLODActor, UHLODProxy* InProxy, const FHierarchicalSimplification& LODSetup);
	
	/** Callback function used for processing finished mesh generation jobs	
	* @param InGuid - Guid of the finished job
	* @param InAssetsToSync - Assets data created by the job
	*/
	HIERARCHICALLODUTILITIES_API void ProcessProxy(const FGuid InGuid, TArray<UObject*>& InAssetsToSync);
	
	/** Returns the callback delegate which will be passed onto ProxyLOD function */
	HIERARCHICALLODUTILITIES_API FCreateProxyDelegate& GetCallbackDelegate();
		
	HIERARCHICALLODUTILITIES_API bool IsProxyGenerationRunning() const;
protected:
	/** Called when the map has changed*/
	HIERARCHICALLODUTILITIES_API void OnMapChange(uint32 MapFlags);

	/** Called when the current level has changed */
	HIERARCHICALLODUTILITIES_API void OnNewCurrentLevel();

	/** Clears the processing data array/map */
	HIERARCHICALLODUTILITIES_API void ClearProcessingData();
protected:
	/** Structure storing the data required during processing */
	struct FProcessData
	{
		/** LODActor instance for which a proxy is generated */
		ALODActor* LODActor;
		/** Proxy mesh where the rendering data is stored */
		UHLODProxy* Proxy;
		/** Array with resulting asset objects from proxy generation (StaticMesh/Materials/Textures) */
		TArray<TStrongObjectPtr<UObject>> AssetObjects;
		/** HLOD settings structure used for creating the proxy */
		FHierarchicalSimplification LODSetup;
	};
private:
	/** Map and array used to store job data */
	TMap<FGuid, FProcessData*> JobActorMap;
	TArray<FProcessData*> ToProcessJobs;
	/** Delegate to pass onto */
	FCreateProxyDelegate CallbackDelegate;	
	/** Critical section to keep JobActorMap/ToProcessJobs access thread-safe */
	FCriticalSection StateLock;	
};
