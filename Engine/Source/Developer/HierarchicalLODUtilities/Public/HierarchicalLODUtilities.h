// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/StaticMesh.h"
#include "IHierarchicalLODUtilities.h"

class AActor;
class AHierarchicalLODVolume;
class ALODActor;
class AWorldSettings;
class FStaticMeshRenderData;
class ULevel;
class UStaticMeshComponent;
struct FHierarchicalSimplification;

/**
* FHierarchicalLODUtilities implementation
*/
class FHierarchicalLODUtilities : public IHierarchicalLODUtilities
{
public:
	~FHierarchicalLODUtilities() = default;
	
	virtual void ExtractSubActorsFromLODActor(AActor* Actor, TArray<AActor*>& InOutActors) override;
	virtual float CalculateScreenSizeFromDrawDistance(const float SphereRadius, const FMatrix& ProjectionMatrix, const float Distance) override;
	virtual float CalculateDrawDistanceFromScreenSize(const float SphereRadius, const float ScreenSize, const FMatrix& ProjectionMatrix) override;
	virtual UPackage* CreateOrRetrieveLevelHLODPackage(const ULevel* InLevel, const uint32 HLODLevelIndex) override;
	virtual UPackage* RetrieveLevelHLODPackage(const ULevel* InLevel, const uint32 HLODLevelIndex) override;
	virtual UHLODProxy* CreateOrRetrieveLevelHLODProxy(const ULevel* InLevel, const uint32 HLODLevelIndex) override;
	virtual UHLODProxy* RetrieveLevelHLODProxy(const ULevel* InLevel, const uint32 HLODLevelIndex) override;
	virtual FString GetWildcardOfHLODPackagesForPackage(const FString& PackageName) override;
	virtual FString GetLevelHLODProxyName(const FString& InLevelPackageName, const uint32 HLODLevelIndex) override;
	virtual bool BuildStaticMeshForLODActor(ALODActor* LODActor, UHLODProxy* Proxy, const FHierarchicalSimplification& LODSetup, UMaterialInterface* InBaseMaterial) override;
	virtual EClusterGenerationError ShouldGenerateCluster(AActor* Actor, const int32 HLODLevelIndex) override;
	virtual ALODActor* GetParentLODActor(const AActor* InActor) override;
	virtual void DestroyCluster(ALODActor* InActor) override;
	virtual ALODActor* CreateNewClusterActor(UWorld* InWorld, const int32 InLODLevel, AWorldSettings* WorldSettings) override;
	virtual ALODActor* CreateNewClusterFromActors(UWorld* InWorld, AWorldSettings* WorldSettings, const TArray<AActor*>& InActors, const int32 InLODLevel = 0) override;
	virtual const bool RemoveActorFromCluster(AActor* InActor) override;
	virtual const bool AddActorToCluster(AActor* InActor, ALODActor* InParentActor) override;
	virtual const bool MergeClusters(ALODActor* TargetCluster, ALODActor* SourceCluster) override;
	virtual const bool AreActorsInSamePersistingLevel(const TArray<AActor*>& InActors) override;
	virtual const bool AreClustersInSameHLODLevel(const TArray<ALODActor*>& InLODActors) override;
	virtual const bool AreActorsInSameHLODLevel(const TArray<AActor*>& InActors) override;
	virtual const bool AreActorsClustered(const TArray<AActor*>& InActors) override;
	virtual const bool IsActorClustered(const AActor* InActor) override;
	virtual void ExcludeActorFromClusterGeneration(AActor* InActor) override;
	virtual void DestroyLODActor(ALODActor* InActor) override;
	virtual void ExtractStaticMeshActorsFromLODActor(ALODActor* LODActor, TArray<AActor*> &InOutActors) override;
	virtual void DeleteLODActorsInHLODLevel(UWorld* InWorld, const int32 HLODLevelIndex) override;
	virtual int32 ComputeStaticMeshLODLevel(const TArray<FStaticMeshSourceModel>& SourceModels, const FStaticMeshRenderData* RenderData, const float ScreenSize) override;
	virtual int32 GetLODLevelForScreenSize(const UStaticMeshComponent* StaticMeshComponent, const float ScreenSize) override;
	virtual AHierarchicalLODVolume* CreateVolumeForLODActor(ALODActor* InLODActor, UWorld* InWorld) override;
	virtual void HandleActorModified(AActor* InActor) override;
	virtual bool IsWorldUsedForStreaming(const UWorld* InWorld) override;
	virtual void CleanStandaloneAssetsInPackage(UPackage* InPackage) override;
};

