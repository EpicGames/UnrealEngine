// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API HIERARCHICALLODUTILITIES_API

class FHierarchicalLODProxyProcessor;
class IHierarchicalLODUtilities;

class IHierarchicalLODUtilitiesModule : public IModuleInterface
{
public:

	virtual FHierarchicalLODProxyProcessor* GetProxyProcessor() = 0;
	virtual IHierarchicalLODUtilities* GetUtilities() = 0;
};

/**
* IHierarchicalLODUtilities module interface
*/
class FHierarchicalLODUtilitiesModule : public IHierarchicalLODUtilitiesModule
{
public:
	UE_API virtual void ShutdownModule() override;
	UE_API virtual void StartupModule() override;

	/** Returns the Proxy processor instance from within this module */
	UE_API virtual FHierarchicalLODProxyProcessor* GetProxyProcessor() override;
	UE_API virtual IHierarchicalLODUtilities* GetUtilities() override;
private:
	FHierarchicalLODProxyProcessor* ProxyProcessor;
	IHierarchicalLODUtilities* Utilities;
};

#undef UE_API
