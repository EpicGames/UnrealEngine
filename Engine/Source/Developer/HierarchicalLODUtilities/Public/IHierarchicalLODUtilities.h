// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/StaticMesh.h"

class AActor;
class AHierarchicalLODVolume;
class ALODActor;
class AWorldSettings;
class FStaticMeshRenderData;
class ULevel;
class UStaticMeshComponent;
struct FHierarchicalSimplification;
class UHLODProxy;

enum class EClusterGenerationError : uint32
{
	None = 0,
	ValidActor = 1 << 1,
	InvalidActor = 1 << 2,
	ActorHiddenInGame = 1 << 3,
	ExcludedActor = 1 << 4,
	LODActor = 1 << 5,
	ActorTooSmall = 1 << 6,
	AlreadyClustered = 1 << 7,
	ComponentHiddenInGame = 1 << 8,
	MoveableComponent = 1 << 9,
	ExcludedComponent = 1 << 10
};

ENUM_CLASS_FLAGS(EClusterGenerationError);

/**
 * IHierarchicalLODUtilities module interface
 */
class IHierarchicalLODUtilities
{
public:	
	virtual ~IHierarchicalLODUtilities() = default;

	/**
	* Recursively retrieves Actors from a LODActor and its child LODActors
	*
	* @param Actor - LODActor instance
	* @param InOutComponents - Will hold the StaticMeshComponents
	*/
	virtual void ExtractSubActorsFromLODActor(AActor* Actor, TArray<AActor*>& InOutActors) = 0;

	/** Computes the Screensize of the given Sphere taking into account the ProjectionMatrix and distance */
	virtual float CalculateScreenSizeFromDrawDistance(const float SphereRadius, const FMatrix& ProjectionMatrix, const float Distance) = 0;

	virtual float CalculateDrawDistanceFromScreenSize(const float SphereRadius, const float ScreenSize, const FMatrix& ProjectionMatrix) = 0;

	/** Creates or retrieves the HLOD package that is created for the given level */
	virtual UPackage* CreateOrRetrieveLevelHLODPackage(const ULevel* InLevel, const uint32 HLODLevelIndex) = 0;

	/** Retrieves the HLOD package that is created for the given level */
	virtual UPackage* RetrieveLevelHLODPackage(const ULevel* InLevel, const uint32 HLODLevelIndex) = 0;

	/** Creates or retrieves the HLOD proxy that is created for the given level */
	virtual UHLODProxy* CreateOrRetrieveLevelHLODProxy(const ULevel* InLevel, const uint32 HLODLevelIndex) = 0;

	/** Retrieves the HLOD proxy that is created for the given level */
	virtual UHLODProxy* RetrieveLevelHLODProxy(const ULevel* InLevel, const uint32 HLODLevelIndex) = 0;

	/** Retrieves a wildcard that includes all possible HLOD packagenames that could be used by the given package. */
	virtual FString GetWildcardOfHLODPackagesForPackage(const FString& PackageName) = 0;

	/** Get the name of a HLOD proxy given the name of a level's package */
	virtual FString GetLevelHLODProxyName(const FString& InLevelPackageName, const uint32 HLODLevelIndex) = 0;

	/**
	* Builds a virtual mesh object for the given LODACtor
	*
	* @param LODActor - Actor to build the mesh for
	* @param Outer - Outer object to store the mesh in
	* @return UStaticMesh*
	*/
	virtual bool BuildStaticMeshForLODActor(ALODActor* LODActor, UHLODProxy* Proxy, const FHierarchicalSimplification& LODSetup, UMaterialInterface* InBaseMaterial) = 0;
	
	/**
	* Returns whether or not the given actor is eligible for creating a HLOD cluster creation
	*
	* @param Actor - Actor to check for if it is eligible for cluster generation
	* @return EClusterGenerationError
	*/
	virtual EClusterGenerationError ShouldGenerateCluster(AActor* Actor, const int32 HLODLevelIndex) = 0;

	/** Returns the ALODActor parent for the given InActor, nullptr if none available */
	virtual ALODActor* GetParentLODActor(const AActor* InActor) = 0;

	/** Deletes the given cluster's data and instance in the world */
	virtual void DestroyCluster(ALODActor* InActor) = 0;

	/** Creates a new cluster actor in the given InWorld with InLODLevel as HLODLevel */
	virtual ALODActor* CreateNewClusterActor(UWorld* InWorld, const int32 InLODLevel, AWorldSettings* WorldSettings) = 0;

	/** Creates a new cluster in InWorld with InActors as sub actors*/
	virtual ALODActor* CreateNewClusterFromActors(UWorld* InWorld, AWorldSettings* WorldSettings, const TArray<AActor*>& InActors, const int32 InLODLevel = 0) = 0;

	/** Removes the given actor from it's parent cluster */
	virtual const bool RemoveActorFromCluster(AActor* InActor) = 0;

	/** Adds an actor to the given cluster*/
	virtual const bool AddActorToCluster(AActor* InActor, ALODActor* InParentActor) = 0;

	/** Merges two clusters together */
	virtual const bool MergeClusters(ALODActor* TargetCluster, ALODActor* SourceCluster) = 0;

	/** Checks if all actors have the same outer world */
	virtual const bool AreActorsInSamePersistingLevel(const TArray<AActor*>& InActors) = 0;

	/** Checks if all clusters are in the same HLOD level */
	virtual const bool AreClustersInSameHLODLevel(const TArray<ALODActor*>& InLODActors) = 0;

	/** Checks if all actors are in the same HLOD level */
	virtual const bool AreActorsInSameHLODLevel(const TArray<AActor*>& InActors) = 0;

	/** Checks if all actors are part of a cluster */
	virtual const bool AreActorsClustered(const TArray<AActor*>& InActors) = 0;

	/** Checks if an actor is clustered*/
	virtual const bool IsActorClustered(const AActor* InActor) = 0;

	/** Excludes an actor from the cluster generation process */
	virtual void ExcludeActorFromClusterGeneration(AActor* InActor) = 0;

	/**
	* Destroys an LODActor instance
	*
	* @param InActor - ALODActor to destroy
	*/
	virtual void DestroyLODActor(ALODActor* InActor) = 0;

	/**
	* Extracts all the virtual Mesh Actors from the given LODActor's SubActors array
	*
	* @param LODActor - LODActors to check the SubActors array for
	* @param InOutActors - Array to fill with virtual Mesh Actors
	*/
	virtual void ExtractStaticMeshActorsFromLODActor(ALODActor* LODActor, TArray<AActor*> &InOutActors) = 0;

	/** Deletes all the ALODActors with the given HLODLevelIndex inside off InWorld */
	virtual void DeleteLODActorsInHLODLevel(UWorld* InWorld, const int32 HLODLevelIndex) = 0;

	/** Computes which LOD level of a Mesh corresponds to the given Distance (calculates closest ScreenSize with distance) */
	virtual int32 ComputeStaticMeshLODLevel(const TArray<FStaticMeshSourceModel>& SourceModels, const FStaticMeshRenderData* RenderData, const float ScreenSize) = 0;

	/** Computes the LODLevel for a StaticMeshComponent taking into account the ScreenSize */
	virtual int32 GetLODLevelForScreenSize(const UStaticMeshComponent* StaticMeshComponent, const float ScreenSize) = 0;
	
	/**
	* Creates a HierarchicalLODVolume using the bounds of a given LODActor
	* @param InLODActor - LODActor to create the volume for
	* @param InWorld - World to spawn the volume in. If null, will spawn in the same level as InLODActor.
	* @return AHierarchicalLODVolume*
	*/
	virtual AHierarchicalLODVolume* CreateVolumeForLODActor(ALODActor* InLODActor, UWorld* InWorld) = 0;

	/**
	* Handles changes in actors for the current world, checks if InActor is part of a HLOD cluster and if so set its dirty-flag
	* @param InActor - Actor to check and find cluster for
	*/
	virtual void HandleActorModified(AActor* InActor) = 0;

	/** Checks whether or not the given InWorld is used as a streaming level by any other World in the level 
	* @param InWorld - World to check whether or not it is used as a streaming level
	*/
	virtual bool IsWorldUsedForStreaming(const UWorld* InWorld) = 0;

	/** 
	 * Cleans any standalone flags for meshes, textures and materials. This ensures that they get garbage collected when not referenced any more.
	 * @param InPackage	The HLOD package to clean
	 */
	virtual void CleanStandaloneAssetsInPackage(UPackage* InPackage) = 0;
};
